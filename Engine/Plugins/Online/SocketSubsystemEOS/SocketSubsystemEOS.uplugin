{
	"FileVersion": 3,

	"FriendlyName": "Socket Subsystem EOS",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Responsible for management of EOS P2P Socket connections.",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,

	"Modules": [
		{
			"Name": "SocketSubsystemEOS",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Android"
			]
		}
	],

	"Plugins": [
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		},
		{
			"Name": "EOSShared",
			"Enabled": true
		}
	]
}
