// Copyright Epic Games, Inc. All Rights Reserved.

#include "InternetAddrEOS.h"
#include "EOSSharedTypes.h"

DEFINE_LOG_CATEGORY(LogSocketSubsystemEOS);

inline uint8 PortToChannel(int32 InPort)
{
	return InPort > 255 ? InPort % 256 : FMath::Clamp(InPort, 0, 255);
}

FInternetAddrEOS::FInternetAddrEOS()
	: ProductUserId(nullptr)
{

}

FInternetAddrEOS::FInternetAddrEOS(const FString& InProductUserId, const FString& InSocketName, const int32 InChannel)
	: FInternetAddrEOS(InProductUserId)
{

}

FInternetAddrEOS::FInternetAddrEOS(const FString& InProductUserId)
	: ProductUserId(nullptr)
{
#if WITH_EOS_P2P
	ProductUserId = EOS_ProductUserId_FromString(TCHAR_TO_UTF8(*InProductUserId));
#endif
}

#if WITH_EOS_P2P
FInternetAddrEOS::FInternetAddrEOS(const EOS_ProductUserId InProductUserId, const FString& InSocketName, const int32 InChannel)
	: FInternetAddrEOS(InProductUserId)
{

}

FInternetAddrEOS::FInternetAddrEOS(const EOS_ProductUserId InProductUserId)
	: ProductUserId(InProductUserId)
{
}
#else
FInternetAddrEOS::FInternetAddrEOS(void* const InProductUserId)
	: ProductUserId(InProductUserId)
{
}
#endif

void FInternetAddrEOS::SetIp(uint32)
{
	UE_LOGF(LogSocketSubsystemEOS, Warning, "Calls to FInternetAddrEOS::SetIp are not valid");
}

void FInternetAddrEOS::SetIp(const TCHAR* InAddr, bool& bIsValid)
{
	bIsValid = false;

	if (InAddr == nullptr)
	{
		return;
	}

	TArray<FString> UrlParts;
	FString FullAddress = InAddr;
	FullAddress.ParseIntoArray(UrlParts, EOS_URL_SEPARATOR, false);
	// Expect URLs to look like "EOS:PUID"
	if (UrlParts.Num() != 2)
	{
		return;
	}
	if (UrlParts[0] != EOS_CONNECTION_URL_PREFIX)
	{
		return;
	}
#if WITH_EOS_P2P
	ProductUserId = EOS_ProductUserId_FromString(TCHAR_TO_UTF8(*UrlParts[1]));
	if (EOS_ProductUserId_IsValid(ProductUserId) == EOS_FALSE)
	{
		return;
	}
#endif

	bIsValid = true;
}

void FInternetAddrEOS::GetIp(uint32& OutAddr) const
{
	OutAddr = 0u;

	UE_LOGF(LogSocketSubsystemEOS, Warning, "Calls to FInternetAddrEOS::GetIp are not valid");
}

bool FInternetAddrEOS::IsPortValid(int32 InPort) const
{
	// There are no ports in EOS connection strings
	return false;
}

void FInternetAddrEOS::SetPort(int32 InPort)
{
	UE_LOGF(LogSocketSubsystemEOS, Warning, "Calls to FInternetAddrEOS::SetPort are not valid");
}

int32 FInternetAddrEOS::GetPort() const
{
	UE_LOGF(LogSocketSubsystemEOS, Warning, "Calls to FInternetAddrEOS::GetPort are not valid");

	return -1;
}

void FInternetAddrEOS::SetRawIp(const TArray<uint8>& RawAddr)
{
	// Need auto here, as might give us different return type depending on size of TCHAR
	auto ConvertedTCHARData = StringCast<TCHAR>(reinterpret_cast<const ANSICHAR*>(RawAddr.GetData()), RawAddr.Num());
	const FString IpAsString = FString::ConstructFromPtrSize(ConvertedTCHARData.Get(), ConvertedTCHARData.Length());

	bool bUnused;
	SetIp(*IpAsString, bUnused);
}

TArray<uint8> FInternetAddrEOS::GetRawIp() const
{
	// We could do this more efficiently, but this was much faster to write.
	const FString StringVersion = ToString(true);

	// Need auto here, as might give us different return type depending on size of TCHAR
	auto ConvertedANSIData = StringCast<ANSICHAR>(*StringVersion, StringVersion.Len());

	TArray<uint8> OutData;
	for (int32 Index = 0; Index < ConvertedANSIData.Length(); ++Index)
	{
		OutData.Add(ConvertedANSIData.Get()[Index]);
	}

	return OutData;
}

void FInternetAddrEOS::SetAnyAddress()
{
}

void FInternetAddrEOS::SetBroadcastAddress()
{
}

void FInternetAddrEOS::SetLoopbackAddress()
{
}

FString FInternetAddrEOS::ToString(bool bAppendPort) const
{
	char PuidBuffer[64];
	int32 BufferLen = 64;
#if WITH_EOS_P2P
	if (EOS_ProductUserId_ToString(ProductUserId, PuidBuffer, &BufferLen) != EOS_EResult::EOS_Success)
#endif
	{
		PuidBuffer[0] = '\0';
	}

	return FString::Printf(TEXT("%s%s%s"), EOS_CONNECTION_URL_PREFIX, EOS_URL_SEPARATOR, UTF8_TO_TCHAR(PuidBuffer));
}

uint32 FInternetAddrEOS::GetTypeHash() const
{
	return GetTypeHashHelper((void*)ProductUserId);
}

bool FInternetAddrEOS::IsValid() const
{
#if WITH_EOS_P2P
	return (EOS_ProductUserId_IsValid(ProductUserId) == EOS_TRUE);
#else
	return false;
#endif
}

TSharedRef<FInternetAddr> FInternetAddrEOS::Clone() const
{
	return MakeShared<FInternetAddrEOS>(*this);
}

SOCKETSUBSYSTEMEOS_API FName FInternetAddrEOS::GetProtocolType() const
{
	static FName TypeName = FName(TEXT("EOS"));
	return TypeName;
}
