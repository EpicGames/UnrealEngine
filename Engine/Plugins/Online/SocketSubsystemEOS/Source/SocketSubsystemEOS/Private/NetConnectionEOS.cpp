// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetConnectionEOS.h"
#include "NetDriverEOS.h"
#include "SocketEOS.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetConnectionEOS)

UNetConnectionEOS::UNetConnectionEOS(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIsPassthrough(false)
	, bHasP2PSession(false)
{
}

void UNetConnectionEOS::InitLocalConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	bIsPassthrough = !InURL.Host.StartsWith(EOS_CONNECTION_URL_PREFIX, ESearchCase::IgnoreCase);
	bHasP2PSession = !bIsPassthrough;
	  
	if (bHasP2PSession)
	{
		DisableAddressResolution();
	}

	Super::InitLocalConnection(InDriver, InSocket, InURL, InState, InMaxPacket, InPacketOverhead);
}

void UNetConnectionEOS::InitRemoteConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, const FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	bIsPassthrough = static_cast<UNetDriverEOS*>(InDriver)->bIsPassthrough;
	bHasP2PSession = !bIsPassthrough;

	if (bHasP2PSession)
	{
		DisableAddressResolution();
	}

	Super::InitRemoteConnection(InDriver, InSocket, InURL, InRemoteAddr, InState, InMaxPacket, InPacketOverhead);
}

void UNetConnectionEOS::CleanUp()
{
	Super::CleanUp();

	if (bHasP2PSession)
	{
		DestroyEOSConnection();
	}
}

void UNetConnectionEOS::DestroyEOSConnection()
{
	FSocket* CurSocket = GetSocket();

	if (CurSocket != nullptr && bHasP2PSession)
	{
		bHasP2PSession = false;

		TSharedPtr<FInternetAddrEOS> RemoteAddrEOS = StaticCastSharedPtr<FInternetAddrEOS>(RemoteAddr);

		if (RemoteAddrEOS.IsValid())
		{
			static_cast<FSocketEOS*>(CurSocket)->Close(*RemoteAddrEOS);
		}
	}
}

