// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IpConnection.h"
#include "NetConnectionEOS.generated.h"

#define UE_API SOCKETSUBSYSTEMEOS_API

UCLASS(MinimalAPI, Transient, Config=Engine)
class UNetConnectionEOS
	: public UIpConnection
{
	GENERATED_BODY()

public:
	UE_API explicit UNetConnectionEOS(const FObjectInitializer& ObjectInitializer);

//~ Begin NetConnection Interface
	UE_API virtual void InitLocalConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	UE_API virtual void InitRemoteConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, const FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	UE_API virtual void CleanUp() override;
//~ End NetConnection Interface

	UE_API void DestroyEOSConnection();

public:
	bool bIsPassthrough;

protected:
	bool bHasP2PSession;
};

#undef UE_API
