// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetDriverEOS.h"
#include "Misc/ConfigCacheIni.h"
#include "NetConnectionEOS.h"
#include "SocketEOS.h"
#include "SocketSubsystemEOS.h"
#include "Engine/Engine.h"
#include "OnlineSubsystemUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetDriverEOS)

UNetDriverEOS::UNetDriverEOS(const FObjectInitializer& ObjectInitializer)
	:UIpNetDriver(ObjectInitializer)
{
	bool bUnused;

	// We check if there is config for bIsUsingP2PSockets for any of the old locations of the plugin
	if (GConfig->GetBool(TEXT("/Script/OnlineSubsystemEOS.NetDriverEOS"), TEXT("bIsUsingP2PSockets"), bUnused, GEngineIni))
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "bIsUsingP2PSockets is deprecated, please remove any related config values");
	}

	if (GConfig->GetBool(TEXT("/Script/SocketSubsystemEOS.NetDriverEOSBase"), TEXT("bIsUsingP2PSockets"), bUnused, GEngineIni))
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "bIsUsingP2PSockets is deprecated, please remove any related config values");
	}
}

bool UNetDriverEOS::IsAvailable() const
{
	// Use passthrough sockets if we are a dedicated server
	if (IsRunningDedicatedServer())
	{
		return false;
	}

	if (ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(EOS_SOCKETSUBSYSTEM))
	{
		return true;
	}

	return false;
}

bool UNetDriverEOS::InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error)
{
	if (bIsPassthrough)
	{
		UE_LOGF(LogSocketSubsystemEOS, Verbose, "Running as pass-through");
		return Super::InitBase(bInitAsClient, InNotify, URL, bReuseAddressAndPort, Error);
	}

	if (!UNetDriver::InitBase(bInitAsClient, InNotify, URL, bReuseAddressAndPort, Error))
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Failed to init driver base");
		return false;
	}

	FSocketSubsystemEOS* const SocketSubsystem = static_cast<FSocketSubsystemEOS*>(GetSocketSubsystem());
	if (!SocketSubsystem)
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Could not get socket subsystem");
		return false;
	}

	// We don't care if our world is null, everything we uses handles it fine
	const UWorld* const MyWorld = FindWorld();

	// Get our local address (proves we're logged in)
	TSharedRef<FInternetAddr> LocalAddress = SocketSubsystem->GetLocalBindAddr(MyWorld, *GLog);
	if (!LocalAddress->IsValid())
	{
		// Not logged in?
		Error = TEXT("Could not bind local address");
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Could not bind local address");
		return false;
	}

	FUniqueSocket NewSocket = SocketSubsystem->CreateUniqueSocket(NAME_DGram, TEXT("UE4"), NAME_None);
	TSharedPtr<FSocket> SharedSocket(NewSocket.Release(), FSocketDeleter(NewSocket.GetDeleter()));
	SetSocketAndLocalAddress(SharedSocket);

	if (GetSocket() == nullptr)
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Could not create socket");
		return false;
	}

	TSharedPtr<FSocketEOS> SharedSocketEOS = StaticCastSharedPtr<FSocketEOS>(SharedSocket);
	check(SharedSocketEOS.IsValid());

	SharedSocketEOS->SetSocketName(GetNetDriverDefinition().ToString());

	TSharedRef<FInternetAddrEOS> EOSLocalAddress = StaticCastSharedRef<FInternetAddrEOS>(LocalAddress);
	
	SharedSocketEOS->SetLocalAddress(*EOSLocalAddress);

	LocalAddr = LocalAddress;

	return true;
}

bool UNetDriverEOS::InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error)
{
	if (!IsAvailable() || !ConnectURL.Host.StartsWith(EOS_CONNECTION_URL_PREFIX, ESearchCase::IgnoreCase))
	{
		UE_LOGF(LogSocketSubsystemEOS, Verbose, "Connecting using IPNetDriver passthrough. ConnectUrl = (%ls)", *ConnectURL.ToString());

		bIsPassthrough = true;
		return Super::InitConnect(InNotify, ConnectURL, Error);
	}

	UE_LOGF(LogSocketSubsystemEOS, Verbose, "Connecting using EOSNetDriver. ConnectUrl = (%ls)", *ConnectURL.ToString());

	if (!InitBase(true, InNotify, ConnectURL, false, Error))
	{
		return false;
	}
	// Reference to our newly created socket
	FSocket* CurSocket = GetSocket();

	// Create an unreal connection to the server
	UNetConnectionEOS* Connection = NewObject<UNetConnectionEOS>(NetConnectionClass);
	check(Connection);

	// Set it as the server connection before anything else so everything knows this is a client
	ServerConnection = Connection;
	Connection->InitLocalConnection(this, CurSocket, ConnectURL, USOCK_Pending);

	CreateInitialClientChannels();

	return true;
}

bool UNetDriverEOS::InitListen(FNetworkNotify* InNotify, FURL& LocalURL, bool bReuseAddressAndPort, FString& Error)
{
	if (!IsAvailable() || LocalURL.HasOption(TEXT("bIsLanMatch")) || LocalURL.HasOption(TEXT("bUseIPSockets")))
	{
		UE_LOGF(LogSocketSubsystemEOS, Verbose, "Init as IPNetDriver listen server. LocalURL = (%ls)", *LocalURL.ToString());

		bIsPassthrough = true;
		return Super::InitListen(InNotify, LocalURL, bReuseAddressAndPort, Error);
	}

	UE_LOGF(LogSocketSubsystemEOS, Verbose, "Init as EOSNetDriver listen server. LocalURL = (%ls)", *LocalURL.ToString());

	if (!InitBase(false, InNotify, LocalURL, bReuseAddressAndPort, Error))
	{
		return false;
	}

	// Bind our specified port if provided
	FSocket* CurSocket = GetSocket();
	if (!CurSocket->Listen(0))
	{
		Error = TEXT("Could not listen");
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Could not listen on socket");
		return false;
	}

	InitConnectionlessHandler();

	UE_LOGF(LogSocketSubsystemEOS, Verbose, "Initialized as an EOSP2P listen server");
	return true;
}

ISocketSubsystem* UNetDriverEOS::GetSocketSubsystem()
{
	if (bIsPassthrough)
	{
		return ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	}
	else
	{
		UWorld* CurrentWorld = FindWorld();
		FSocketSubsystemEOS* DefaultSocketSubsystem = static_cast<FSocketSubsystemEOS*>(ISocketSubsystem::Get(EOS_SOCKETSUBSYSTEM));
		return DefaultSocketSubsystem->GetSocketSubsystemForWorld(CurrentWorld);
	}
}

void UNetDriverEOS::Shutdown()
{
	UE_LOGF(LogSocketSubsystemEOS, Verbose, "Shutting down NetDriver");

	Super::Shutdown();

	// Kill our P2P sessions now, instead of when garbage collection kicks in later
	if (!bIsPassthrough)
	{
		if (UNetConnectionEOS* const EOSServerConnection = Cast<UNetConnectionEOS>(ServerConnection))
		{
			EOSServerConnection->DestroyEOSConnection();
		}
		for (UNetConnection* Client : ClientConnections)
		{
			if (UNetConnectionEOS* const EOSClient = Cast<UNetConnectionEOS>(Client))
			{
				EOSClient->DestroyEOSConnection();
			}
		}
	}
}

int UNetDriverEOS::GetClientPort()
{
	if (bIsPassthrough)
	{
		return Super::GetClientPort();
	}

	// Starting range of dynamic/private/ephemeral ports
	return 49152;
}

UWorld* UNetDriverEOS::FindWorld() const
{
	UWorld* MyWorld = GetWorld();

	// If we don't have a world, we may be a pending net driver
	if (!MyWorld && GEngine)
	{
		if (FWorldContext* WorldContext = GEngine->GetWorldContextFromPendingNetGameNetDriver(this))
		{
			MyWorld = WorldContext->World();
		}
	}

	return MyWorld;
}


