// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketEOS.h"
#include "SocketSubsystemEOS.h"
#include "EOSShared.h"
#include "Misc/ConfigCacheIni.h"

#if WITH_EOS_P2P
	#include "eos_p2p.h"
#endif

FSocketEOS::FSocketEOS(FSocketSubsystemEOS& InSocketSubsystem, const FString& InSocketDescription)
	: FSocket(ESocketType::SOCKTYPE_Datagram, InSocketDescription, NAME_None)
	, SocketSubsystem(InSocketSubsystem)
	, bIsListening(false)
#if WITH_EOS_P2P
	, ConnectNotifyCallback(nullptr)
	, ConnectNotifyId(EOS_INVALID_NOTIFICATIONID)
	, ClosedNotifyCallback(nullptr)
	, ClosedNotifyId(EOS_INVALID_NOTIFICATIONID)
#endif
{
	CallbackAliveTracker = MakeShared<FCallbackBase>();

#if WITH_EOS_P2P
	FString PacketReliabilityTypeStr;
	if (GConfig->GetString(TEXT("SocketSubsystemEOS"), TEXT("PacketReliabilityType"), PacketReliabilityTypeStr, GEngineIni))
	{
		EOS_EPacketReliability PacketReliabilityType;
		if (LexFromString(PacketReliabilityType, *PacketReliabilityTypeStr))
		{
			PacketReliability = PacketReliabilityType;
		}
	}

	if (IEOSSDKManager* EOSSDKManager = IEOSSDKManager::Get())
	{
		NetworkStatusChangedDelegateHandle = EOSSDKManager->OnNetworkStatusChanged.AddRaw(this, &FSocketEOS::OnNetworkStatusChanged);
	}
#endif
}

FSocketEOS::~FSocketEOS()
{
	Close();

	SocketSubsystem.UnbindSocketName(SocketName);
	LocalAddress = FInternetAddrEOS();
	CallbackAliveTracker = nullptr;

#if WITH_EOS_P2P
	if (IEOSSDKManager* EOSSDKManager = IEOSSDKManager::Get())
	{
		EOSSDKManager->OnNetworkStatusChanged.Remove(NetworkStatusChangedDelegateHandle);
	}
#endif
}

bool FSocketEOS::Shutdown(ESocketShutdownMode Mode)
{
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

#if WITH_EOS_P2P
void FSocketEOS::OnNetworkStatusChanged(EOS_ENetworkStatus OldNetworkStatus, EOS_ENetworkStatus NewNetworkStatus)
{
	// Only needed if we are listening on a socket, otherwise nothing needs to be done
	if (bIsListening)
	{
		// Network status changes from an offline state to online - need to rebind P2P connection request notification 
		if (OldNetworkStatus != EOS_ENetworkStatus::EOS_NS_Online && NewNetworkStatus == EOS_ENetworkStatus::EOS_NS_Online)
		{
			Listen(0);
		}
		// Network status changes from online to another state - clean up what was set on Listen()
		else if (OldNetworkStatus == EOS_ENetworkStatus::EOS_NS_Online && NewNetworkStatus != EOS_ENetworkStatus::EOS_NS_Online)
		{
			// This is a special case where the P2P interface in the EOS SDK unbinds notifiers for us. We need to cleanup bound functions and notification Ids
			ConnectNotifyId = EOS_INVALID_NOTIFICATIONID;
			ClosedNotifyId = EOS_INVALID_NOTIFICATIONID;
			ConnectNotifyCallback = nullptr;
			ClosedNotifyCallback = nullptr;
		}
	}	
}
#endif

bool FSocketEOS::Close()
{
	check(IsInGameThread() && "p2p does not support multithreading");

#if WITH_EOS_P2P
	if (ConnectNotifyId != EOS_INVALID_NOTIFICATIONID)
	{
		EOS_P2P_RemoveNotifyPeerConnectionRequest(SocketSubsystem.GetP2PHandle(), ConnectNotifyId);
	}
	delete ConnectNotifyCallback;
	ConnectNotifyCallback = nullptr;
	if (ClosedNotifyId != EOS_INVALID_NOTIFICATIONID)
	{
		EOS_P2P_RemoveNotifyPeerConnectionClosed(SocketSubsystem.GetP2PHandle(), ClosedNotifyId);
	}
	delete ClosedNotifyCallback;
	ClosedNotifyCallback = nullptr;

	EOS_P2P_SocketId SocketId = { };
	SocketId.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_SOCKETID_API_LATEST, 1);
	FCStringAnsi::Strcpy(SocketId.SocketName, TCHAR_TO_UTF8(*SocketName));

	EOS_P2P_CloseConnectionsOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_CLOSECONNECTIONS_API_LATEST, 1);
	Options.LocalUserId = SocketSubsystem.GetLocalUserId();
	Options.SocketId = &SocketId;

	EOS_EResult Result = EOS_P2P_CloseConnections(SocketSubsystem.GetP2PHandle(), &Options);

	UE_LOGF(LogSocketSubsystemEOS, Log, "Closing socket (%ls) with result (%ls)", *SocketName, *LexToString(Result));

	ClosedRemotes.Empty();
#endif
	return true;
}

bool FSocketEOS::Bind(const FInternetAddr& Addr)
{
	check(IsInGameThread() && "p2p does not support multithreading");

	if (!Addr.IsValid())
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Attempted to bind to invalid address. Address = (%ls)", *Addr.ToString(true));
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EADDRNOTAVAIL);
		return false;
	}

	if (!SocketSubsystem.BindSocketName(SocketName))
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Attempted to bind a socket to a port in use. NewAddress = (%ls)", *Addr.ToString(true));
		// BindPort sets our LastSocketError
		return false;
	}

	UE_LOGF(LogSocketSubsystemEOS, Verbose, "Successfully bound socket (%ls) to address (%ls)", *SocketName, *Addr.ToString(true));
	return true;
}

bool FSocketEOS::Connect(const FInternetAddr& Addr)
{
	/** Not supported - connectionless (UDP) only */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FSocketEOS::Listen(int32)
{
	check(IsInGameThread() && "p2p does not support multithreading");

#if WITH_EOS_P2P
	// Add listener for inbound connections
	EOS_P2P_SocketId SocketId = { };
	SocketId.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_SOCKETID_API_LATEST, 1);
	FCStringAnsi::Strcpy(SocketId.SocketName, TCHAR_TO_UTF8(*SocketName));

	EOS_P2P_AddNotifyPeerConnectionRequestOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_ADDNOTIFYPEERCONNECTIONREQUEST_API_LATEST, 1);
	Options.LocalUserId = SocketSubsystem.GetLocalUserId();
	Options.SocketId = &SocketId;

	ConnectNotifyCallback = new FConnectNotifyCallback(CallbackAliveTracker);
	ConnectNotifyCallback->CallbackLambda = [this](const EOS_P2P_OnIncomingConnectionRequestInfo* Info)
	{
		char PuidBuffer[64];
		int32 BufferLen = 64;
		if (EOS_ProductUserId_ToString(Info->RemoteUserId, PuidBuffer, &BufferLen) != EOS_EResult::EOS_Success)
		{
			PuidBuffer[0] = '\0';
		}
		FString RemoteUser(PuidBuffer);

		if (Info->LocalUserId == SocketSubsystem.GetLocalUserId() && FCStringAnsi::Stricmp(Info->SocketId->SocketName, TCHAR_TO_UTF8(*SocketName)) == 0)
		{
			// In case they disconnected and then reconnected, remove them from our closed list
			FInternetAddrEOS RemoteAddress(Info->RemoteUserId);
			ClosedRemotes.Remove(RemoteAddress);

			EOS_P2P_SocketId SocketId = { };
			SocketId.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_P2P_SOCKETID_API_LATEST, 1);
			FCStringAnsi::Strcpy(SocketId.SocketName, Info->SocketId->SocketName);

			EOS_P2P_AcceptConnectionOptions Options = { };
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_P2P_ACCEPTCONNECTION_API_LATEST, 1);
			Options.LocalUserId = SocketSubsystem.GetLocalUserId();
			Options.RemoteUserId = Info->RemoteUserId;
			Options.SocketId = &SocketId;
			UE_LOGF(LogSocketSubsystemEOS, VeryVerbose, "Calling EOS_P2P_AcceptConnection SocketId=[%s]", SocketId.SocketName);
			EOS_EResult AcceptResult = EOS_P2P_AcceptConnection(SocketSubsystem.GetP2PHandle(), &Options);
			UE_CLOGF(AcceptResult == EOS_EResult::EOS_Success, LogSocketSubsystemEOS, Log, "EOS_P2P_AcceptConnection RemoteUser=[%ls] SocketId=[%s] Result=[%ls]", *RemoteUser, SocketId.SocketName, *LexToString(AcceptResult));
			UE_CLOGF(AcceptResult != EOS_EResult::EOS_Success, LogSocketSubsystemEOS, Error, "EOS_P2P_AcceptConnection RemoteUser=[%ls] SocketId=[%s] Result=[%ls]", *RemoteUser, SocketId.SocketName, *LexToString(AcceptResult));
		}
		else
		{
			UE_LOGF(LogSocketSubsystemEOS, Warning, "Ignoring connection request from RemoteUser=[%ls] SocketId=[%s]", *RemoteUser, Info->SocketId->SocketName);
		}
	};
	ConnectNotifyId = EOS_P2P_AddNotifyPeerConnectionRequest(SocketSubsystem.GetP2PHandle(), &Options, ConnectNotifyCallback, ConnectNotifyCallback->GetCallbackPtr());

	// Need to handle closures too
	RegisterClosedNotification();
#endif

	bIsListening = true;

	return true;
}

bool FSocketEOS::WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime)
{
	/** Not supported - connectionless (UDP) only */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FSocketEOS::HasPendingData(uint32& PendingDataSize)
{
	check(IsInGameThread() && "p2p does not support multithreading");

	PendingDataSize = 0;

#if WITH_EOS_P2P
	EOS_P2P_GetNextReceivedPacketSizeOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_GETNEXTRECEIVEDPACKETSIZE_API_LATEST, 2);
	Options.LocalUserId = SocketSubsystem.GetLocalUserId();
	uint8 Channel = GetTypeHash(SocketName);
	Options.RequestedChannel = &Channel;

	EOS_EResult Result = EOS_P2P_GetNextReceivedPacketSize(SocketSubsystem.GetP2PHandle(), &Options, &PendingDataSize);
	if (Result == EOS_EResult::EOS_NotFound)
	{
		return false;
	}
	if (Result != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "EOS_P2P_GetNextReceivedPacketSize for socket [%ls] (Channel [%d]) returned result code = [%ls]", *SocketName, Channel, *LexToString(Result));

		// @todo joeg - map EOS codes to UE4's
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EINVAL);
		return false;
	}

	return true;
#else
	return false;
#endif
}

FSocket* FSocketEOS::Accept(const FString& InSocketDescription)
{
	/** Not supported - connectionless (UDP) only */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return nullptr;
}

FSocket* FSocketEOS::Accept(FInternetAddr& OutAddr, const FString& InSocketDescription)
{
	/** Not supported - connectionless (UDP) only */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return nullptr;
}

bool FSocketEOS::SendTo(const uint8* Data, int32 Count, int32& OutBytesSent, const FInternetAddr& Destination)
{
	check(IsInGameThread() && "p2p does not support multithreading");

	OutBytesSent = 0;

	if (!SocketSubsystem.GetSocketSubsystemEOSUtils()->IsLoggedIn())
	{
		UE_LOGF(LogSocketSubsystemEOS, VeryVerbose, "FSocketEOS::SendTo - Local user is logged out");

		return false;
	}

	if (!Destination.IsValid())
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Unable to send data, invalid DestinationAddress=[%ls]", *Destination.ToString(true));

		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EADDRNOTAVAIL);
		return false;
	}

#if WITH_EOS_P2P
	if (Count > EOS_P2P_MAX_PACKET_SIZE)
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Unable to send data, data over maximum size. Amount=[%d/%d] DestinationAddress=[%ls]", Count, EOS_P2P_MAX_PACKET_SIZE, *Destination.ToString(true));

		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EMSGSIZE);
		return false;
	}

	if (Count < 0)
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Unable to send data, data invalid. Amount=[%d/%d] DestinationAddress=[%ls]", Count, EOS_P2P_MAX_PACKET_SIZE, *Destination.ToString(true));

		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EINVAL);
		return false;
	}
#endif 

	if (Data == nullptr && Count != 0)
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Unable to send data, data invalid. DestinationAddress=[%ls]", *Destination.ToString(true));

		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EINVAL);
		return false;
	}

	const FInternetAddrEOS& DestinationAddress = static_cast<const FInternetAddrEOS&>(Destination);

	// Check for sending to an address we explicitly closed
	if (WasClosed(DestinationAddress))
	{
		UE_LOGF(LogSocketSubsystemEOS, Warning, "Unable to send data to closed connection. DestinationAddress=[%ls]", *Destination.ToString(true));

		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_ECONNREFUSED);
		return false;
	}

#if WITH_EOS_P2P
	// Need to handle closures if we are a client and the server closes down on us
	RegisterClosedNotification();

	EOS_P2P_SocketId SocketId = { };
	SocketId.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_SOCKETID_API_LATEST, 1);
	FCStringAnsi::Strcpy(SocketId.SocketName, TCHAR_TO_UTF8(*SocketName));

	EOS_P2P_SendPacketOptions Options = { };
	Options.ApiVersion = 3;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_SENDPACKET_API_LATEST, 3);
	Options.LocalUserId = SocketSubsystem.GetLocalUserId();
	Options.RemoteUserId = DestinationAddress.GetProductUserId();
	Options.SocketId = &SocketId;
	Options.bAllowDelayedDelivery = EOS_TRUE;
	Options.Reliability = PacketReliability;
	Options.Channel = GetTypeHash(SocketName);
	Options.DataLengthBytes = Count;
	Options.Data = Data;

	UE_LOGF(LogSocketSubsystemEOS, VeryVerbose, "Calling EOS_P2P_SendPacket DestinationAddress=[%ls] SocketId=[%s]", *DestinationAddress.ToString(true), SocketId.SocketName);
	EOS_EResult SendResult = EOS_P2P_SendPacket(SocketSubsystem.GetP2PHandle(), &Options);
	if (SendResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogSocketSubsystemEOS, Error, "EOS_P2P_SendPacket DestinationAddress=[%ls] SocketId=[%s] Result=[%ls]", *DestinationAddress.ToString(true), SocketId.SocketName, *LexToString(SendResult));

		// @todo joeg - map EOS codes to UE4's
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EINVAL);
		return false;
	}
	UE_LOGF(LogSocketSubsystemEOS, VeryVerbose, "EOS_P2P_SendPacket DestinationAddress=[%ls] SocketId=[%s] Result=[%ls]", *DestinationAddress.ToString(true), SocketId.SocketName,*LexToString(SendResult));
	OutBytesSent = Count;
	return true;
#else
	return false;
#endif
}

bool FSocketEOS::Send(const uint8* Data, int32 Count, int32& BytesSent)
{
	/** Not supported - connectionless (UDP) only */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	BytesSent = 0;
	return false;
}

bool FSocketEOS::RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags)
{
	check(IsInGameThread() && "p2p does not support multithreading");
	BytesRead = 0;

	if (!SocketSubsystem.GetSocketSubsystemEOSUtils()->IsLoggedIn())
	{
		UE_LOGF(LogSocketSubsystemEOS, VeryVerbose, "FSocketEOS::RecvFrom - Local user is logged out");

		return false;
	}
	
	if (BufferSize < 0)
	{
		UE_LOGF(LogSocketSubsystemEOS, Error, "Unable to receive data, receiving buffer was invalid. BufferSize=[%d]", BufferSize);

		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EINVAL);
		return false;
	}

	if (Flags != ESocketReceiveFlags::None)
	{
		// We do not support peaking / blocking until a packet comes
		UE_LOGF(LogSocketSubsystemEOS, Error, "Socket receive Flags=[%d] are not supported", int32(Flags));

		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
		return false;
	}

#if WITH_EOS_P2P
	EOS_P2P_ReceivePacketOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_RECEIVEPACKET_API_LATEST, 2);
	Options.LocalUserId = SocketSubsystem.GetLocalUserId();
	Options.MaxDataSizeBytes = BufferSize;
	uint8 Channel = GetTypeHash(SocketName);
	Options.RequestedChannel = &Channel;

	EOS_ProductUserId RemoteUserId = nullptr;
	EOS_P2P_SocketId SocketId;
	
	UE_LOGF(LogSocketSubsystemEOS, VeryVerbose, "Calling EOS_P2P_ReceivePacket RequestedChannel=[%lld]", Options.RequestedChannel);
	EOS_EResult ReceiveResult = EOS_P2P_ReceivePacket(SocketSubsystem.GetP2PHandle(), &Options, &RemoteUserId, &SocketId, &Channel, Data, (uint32*)&BytesRead);
	if (ReceiveResult == EOS_EResult::EOS_NotFound)
	{
		// No data to read
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EWOULDBLOCK);
		return false;
	}
	else if (ReceiveResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogSocketSubsystemEOS, Error, "EOS_P2P_ReceivePacket RequestedChannel=[%lld] Result=[%ls]", Options.RequestedChannel, *LexToString(ReceiveResult));

		// @todo joeg - map EOS codes to UE4's
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EINVAL);
		return false;
	}

	FInternetAddrEOS& SourceAddress = static_cast<FInternetAddrEOS&>(Source);
	SourceAddress.SetProductUserId(RemoteUserId);

	return true;
#else
	return false;
#endif
}

bool FSocketEOS::Recv(uint8* Data, int32 BufferSize, int32& BytesRead, ESocketReceiveFlags::Type Flags)
{
	BytesRead = 0;
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FSocketEOS::Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime)
{
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

ESocketConnectionState FSocketEOS::GetConnectionState()
{
	return ESocketConnectionState::SCS_NotConnected;
}

void FSocketEOS::GetAddress(FInternetAddr& OutAddr)
{
	OutAddr = LocalAddress;
}

bool FSocketEOS::GetPeerAddress(FInternetAddr& OutAddr)
{
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FSocketEOS::SetNonBlocking(bool bIsNonBlocking)
{
	return true;
}

bool FSocketEOS::SetBroadcast(bool bAllowBroadcast)
{
	return true;
}

bool FSocketEOS::SetNoDelay(bool bIsNoDelay)
{
	return true;
}

bool FSocketEOS::JoinMulticastGroup(const FInternetAddr& GroupAddress)
{
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FSocketEOS::LeaveMulticastGroup(const FInternetAddr& GroupAddress)
{
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FSocketEOS::SetMulticastLoopback(bool bLoopback)
{
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FSocketEOS::SetMulticastTtl(uint8 TimeToLive)
{
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FSocketEOS::JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FSocketEOS::LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FSocketEOS::SetMulticastInterface(const FInternetAddr& InterfaceAddress)
{
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FSocketEOS::SetReuseAddr(bool bAllowReuse)
{
	return true;
}

bool FSocketEOS::SetLinger(bool bShouldLinger, int32 Timeout)
{
	return true;
}

bool FSocketEOS::SetRecvErr(bool bUseErrorQueue)
{
	return true;
}

bool FSocketEOS::SetSendBufferSize(int32 Size, int32& NewSize)
{
	return true;
}

bool FSocketEOS::SetReceiveBufferSize(int32 Size, int32& NewSize)
{
	return true;
}

int32 FSocketEOS::GetPortNo()
{
	return GetTypeHash(SocketName);
}

void FSocketEOS::SetLocalAddress(const FInternetAddrEOS& InLocalAddress)
{
	LocalAddress = InLocalAddress;
}

void FSocketEOS::SetSocketName(const FString& InSocketName)
{
	SocketName = InSocketName;
}

bool FSocketEOS::Close(const FInternetAddrEOS& RemoteAddress)
{
	check(IsInGameThread() && "p2p does not support multithreading");

	if (!RemoteAddress.IsValid())
	{
		UE_LOGF(LogSocketSubsystemEOS, Error, "Unable to close socket with RemoteAddress=[%ls] as it is invalid", *RemoteAddress.ToString(true));
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EADDRNOTAVAIL);
		return false;
	}

#if WITH_EOS_P2P
	// So we don't reopen a connection by sending to it
	ClosedRemotes.Add(RemoteAddress);

	EOS_P2P_SocketId SocketId = { };
	SocketId.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_SOCKETID_API_LATEST, 1);
	FCStringAnsi::Strcpy(SocketId.SocketName, TCHAR_TO_UTF8(*SocketName));

	EOS_P2P_CloseConnectionOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_CLOSECONNECTION_API_LATEST, 1);
	Options.LocalUserId = SocketSubsystem.GetLocalUserId();
	Options.RemoteUserId = RemoteAddress.GetProductUserId();
	Options.SocketId = &SocketId;

	UE_LOGF(LogSocketSubsystemEOS, VeryVerbose, "Calling EOS_P2P_CloseConnection RemoteAddress=[%ls] SocketId=[%s])", *RemoteAddress.ToString(true), SocketId.SocketName);
	EOS_EResult CloseResult = EOS_P2P_CloseConnection(SocketSubsystem.GetP2PHandle(), &Options);
	if (CloseResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogSocketSubsystemEOS, Error, "EOS_P2P_CloseConnection RemoteAddress=[%ls] SocketId=[%s] Result=[%ls]", *RemoteAddress.ToString(true), SocketId.SocketName, *LexToString(CloseResult));

		// @todo joeg - map EOS codes to UE4's
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EINVAL);
		return false;
	}

	UE_LOGF(LogSocketSubsystemEOS, Log, "EOS_P2P_CloseConnection RemoteAddress=[%ls] SocketId=[%s] Result=[%ls]", *RemoteAddress.ToString(true), SocketId.SocketName, *LexToString(CloseResult));
	return true;
#else
	return false;
#endif
}

void FSocketEOS::RegisterClosedNotification()
{
#if WITH_EOS_P2P
	if (ClosedNotifyId != EOS_INVALID_NOTIFICATIONID)
	{
		// Already listening for these events so ignore
		return;
	}
	
	EOS_P2P_SocketId SocketId = { };
	SocketId.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_SOCKETID_API_LATEST, 1);
	FCStringAnsi::Strcpy(SocketId.SocketName, TCHAR_TO_UTF8(*SocketName));

	EOS_P2P_AddNotifyPeerConnectionClosedOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_P2P_ADDNOTIFYPEERCONNECTIONCLOSED_API_LATEST, 1);
	Options.LocalUserId = SocketSubsystem.GetLocalUserId();
	Options.SocketId = &SocketId;

	ClosedNotifyCallback = new FClosedNotifyCallback(CallbackAliveTracker);
	ClosedNotifyCallback->CallbackLambda = [this](const EOS_P2P_OnRemoteConnectionClosedInfo* Info)
	{
		// Add this connection to the list of closed ones
		FInternetAddrEOS RemoteAddress(Info->RemoteUserId);
		ClosedRemotes.Add(RemoteAddress);
	};
	ClosedNotifyId = EOS_P2P_AddNotifyPeerConnectionClosed(SocketSubsystem.GetP2PHandle(), &Options, ClosedNotifyCallback, ClosedNotifyCallback->GetCallbackPtr());
#endif
}
