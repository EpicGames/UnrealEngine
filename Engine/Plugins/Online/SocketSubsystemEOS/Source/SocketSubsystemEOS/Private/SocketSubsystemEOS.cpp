// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystemEOS.h"

#include "EOSShared.h"
#include "Misc/ConfigCacheIni.h"
#include "OnlineSubsystemUtils.h"
#include "SocketEOS.h"
#include "SocketSubsystemModule.h"

#if WITH_EOS_P2P
#include "eos_sdk.h"
#include "eos_p2p.h"

namespace
{

	static EOS_ERelayControl LexFromString(const FString& Value)
	{
		if (Value == TEXT("NoRelays"))
		{
			return EOS_ERelayControl::EOS_RC_NoRelays;
		}
		if (Value == TEXT("AllowRelays"))
		{
			return EOS_ERelayControl::EOS_RC_AllowRelays;
		}
		if (Value == TEXT("ForceRelays"))
		{
			return EOS_ERelayControl::EOS_RC_ForceRelays;
		}

		checkNoEntry();
		return EOS_ERelayControl::EOS_RC_AllowRelays;
	}

}
#endif // WITH_EOS_P2P

TArray<FSocketSubsystemEOS*> FSocketSubsystemEOS::SocketSubsystemEOSInstances;
TMap<UWorld*, FSocketSubsystemEOS*> FSocketSubsystemEOS::SocketSubsystemEOSPerWorldMap;

FSocketSubsystemEOS::FSocketSubsystemEOS(IEOSPlatformHandlePtr InPlatformHandle, ISocketSubsystemEOSUtilsPtr InUtils)
	: Utils(InUtils)
	, LastSocketError(ESocketErrors::SE_NO_ERROR)
{
#if WITH_EOS_P2P
	bool bEnable = true;
	GConfig->GetBool(TEXT("SocketSubsystemEOS"), TEXT("bEnable"), bEnable, GEngineIni);

	if (bEnable)
	{
		P2PHandle = EOS_Platform_GetP2PInterface(*InPlatformHandle);
		if (P2PHandle == nullptr)
		{
			UE_LOGF(LogSocketSubsystemEOS, Error, "FOnlineSubsystemEOS: failed to init EOS platform, couldn't get p2p handle");
		}
	}
#endif // WITH_EOS_P2P
}

FSocketSubsystemEOS::~FSocketSubsystemEOS()
{
	Utils = nullptr;
}

FSocketSubsystemEOS* FSocketSubsystemEOS::GetSocketSubsystemForWorld(UWorld* InWorld)
{
	FSocketSubsystemEOS** Result = SocketSubsystemEOSPerWorldMap.Find(InWorld);

	if(!Result)
	{
		for (FSocketSubsystemEOS* SocketSubsystem : SocketSubsystemEOSInstances)
		{
			const UWorld* NewWorld = GetWorldForOnline(SocketSubsystem->Utils->GetSubsystemInstanceName());

			if (NewWorld == InWorld)
			{
				SocketSubsystemEOSPerWorldMap.Add(InWorld, SocketSubsystem);
				return SocketSubsystem;
			}
		}
	}

	return Result ? *Result : nullptr;
}

bool FSocketSubsystemEOS::Init(FString& Error)
{
	SocketSubsystemEOSInstances.Add(this);

	FSocketSubsystemModule& SocketSubsystem = FModuleManager::LoadModuleChecked<FSocketSubsystemModule>("Sockets");
	SocketSubsystem.RegisterSocketSubsystem(EOS_SOCKETSUBSYSTEM, this, false);

#if WITH_EOS_P2P
	if (P2PHandle)
	{
		FString RelayControlStr;
		GConfig->GetString(TEXT("SocketSubsystemEOS"), TEXT("RelayControl"), RelayControlStr, GEngineIni);
		if (!RelayControlStr.IsEmpty())
		{
			EOS_P2P_SetRelayControlOptions Options = {};
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_P2P_SETRELAYCONTROL_API_LATEST, 1);

			Options.RelayControl = LexFromString(RelayControlStr);

			EOS_EResult Result = EOS_P2P_SetRelayControl(P2PHandle, &Options);
			if (Result != EOS_EResult::EOS_Success && Result != EOS_EResult::EOS_NoChange)
			{
				UE_LOGF(LogSocketSubsystemEOS, Warning, "[FSocketSubsystemEOS::Init] EOS_P2P_SetRelayControl failed with result [%ls]", *LexToString(Result));
			}
		}
	}
#endif // WITH_EOS_P2P

	return true;
}

void FSocketSubsystemEOS::Shutdown()
{
	RemoveFromStaticContainers();

	// Destruct our sockets before we finish destructing, as they maintain a reference to us
	TrackedSockets.Reset();

	if (FSocketSubsystemModule* SocketSubsystem = FModuleManager::GetModulePtr<FSocketSubsystemModule>("Sockets"))
	{
		SocketSubsystem->UnregisterSocketSubsystem(EOS_SOCKETSUBSYSTEM);
	}
}

void FSocketSubsystemEOS::RemoveFromStaticContainers()
{
	for (TMap<UWorld*, FSocketSubsystemEOS*>::TIterator Iter(SocketSubsystemEOSPerWorldMap); Iter; ++Iter)
	{
		if (Iter.Value() == this)
		{
			Iter.RemoveCurrent();
		}
	}

	SocketSubsystemEOSInstances.Remove(this);
}

FSocket* FSocketSubsystemEOS::CreateSocket(const FName& SocketTypeName, const FString& SocketDescription, const FName& /*unused*/)
{
	return TrackedSockets.Emplace_GetRef(MakeUnique<FSocketEOS>(*this, SocketDescription)).Get();
}

FResolveInfoCached* FSocketSubsystemEOS::CreateResolveInfoCached(TSharedPtr<FInternetAddr> Addr) const
{
	return nullptr;
}

void FSocketSubsystemEOS::DestroySocket(FSocket* Socket)
{
	for (auto It = TrackedSockets.CreateIterator(); It; ++It)
	{
		if (It->IsValid() && It->Get() == Socket)
		{
			It.RemoveCurrent();
			return;
		}
	}
}

FAddressInfoResult FSocketSubsystemEOS::GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName, EAddressInfoFlags /*unused*/, const FName /*unused*/, ESocketType /*unused*/)
{
	return FAddressInfoResult(HostName, ServiceName);
}

bool FSocketSubsystemEOS::RequiresChatDataBeSeparate()
{
	return false;
}

bool FSocketSubsystemEOS::RequiresEncryptedPackets()
{
	return false;
}

bool FSocketSubsystemEOS::GetHostName(FString& HostName)
{
	return false;
}

TSharedRef<FInternetAddr> FSocketSubsystemEOS::CreateInternetAddr()
{
	return MakeShared<FInternetAddrEOS>();
}

TSharedPtr<FInternetAddr> FSocketSubsystemEOS::GetAddressFromString(const FString& InString)
{
	bool bUnused;
	TSharedPtr<FInternetAddrEOS> NewAddress = StaticCastSharedRef<FInternetAddrEOS>(CreateInternetAddr());
	NewAddress->SetIp(*InString, bUnused);
	return NewAddress;
}

bool FSocketSubsystemEOS::HasNetworkDevice()
{
	return true;
}

const TCHAR* FSocketSubsystemEOS::GetSocketAPIName() const
{
	return TEXT("p2pSocketsEOS");
}

ESocketErrors FSocketSubsystemEOS::GetLastErrorCode()
{
	return TranslateErrorCode(LastSocketError);
}

ESocketErrors FSocketSubsystemEOS::TranslateErrorCode(int32 Code)
{
	return static_cast<ESocketErrors>(Code);
}

bool FSocketSubsystemEOS::GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses)
{
	TSharedRef<FInternetAddr> AdapterAddress = GetLocalBindAddr(nullptr, *GLog);
	OutAddresses.Add(AdapterAddress);
	return true;
}

TArray<TSharedRef<FInternetAddr>> FSocketSubsystemEOS::GetLocalBindAddresses()
{
	TArray<TSharedRef<FInternetAddr>> OutAddresses;
	OutAddresses.Add(GetLocalBindAddr(nullptr, *GLog));
	return OutAddresses;
}

TSharedRef<FInternetAddr> FSocketSubsystemEOS::GetLocalBindAddr(FOutputDevice& Out)
{
	return GetLocalBindAddr(nullptr, Out);
}

#if WITH_EOS_P2P
EOS_HP2P FSocketSubsystemEOS::GetP2PHandle()
{
	check(P2PHandle != nullptr);
	return P2PHandle;
}

EOS_ProductUserId FSocketSubsystemEOS::GetLocalUserId()
{
	return Utils->GetLocalUserId();
}
#endif // WITH_EOS_P2P

TSharedRef<FInternetAddr> FSocketSubsystemEOS::GetLocalBindAddr(const UWorld* const OwningWorld, FOutputDevice& Out)
{
	TSharedRef<FInternetAddrEOS> BoundAddr = MakeShared<FInternetAddrEOS>();

#if WITH_EOS_P2P
	EOS_ProductUserId LocalUserId = GetLocalUserId();
	if (LocalUserId != nullptr)
	{
		BoundAddr->SetProductUserId(LocalUserId);
	}
	else
	{
		UE_LOGF(LogSocketSubsystemEOS, Error, "No local user set in LocalBindAddr");
	}
#endif // WITH_EOS_P2P

	return BoundAddr;
}

bool FSocketSubsystemEOS::IsSocketWaitSupported() const
{
	return false;
}

void FSocketSubsystemEOS::SetLastSocketError(const ESocketErrors NewSocketError)
{
	LastSocketError = NewSocketError;
}

bool FSocketSubsystemEOS::BindSocketName(const FString& SocketName)
{
	if (BoundSocketNames.Contains(SocketName))
	{
		SetLastSocketError(ESocketErrors::SE_EADDRINUSE);
		return false;
	}

	BoundSocketNames.Add(SocketName);
	return true;
}

bool FSocketSubsystemEOS::UnbindSocketName(const FString& SocketName)
{
	// Remove our port and check if we had it bound
	if (BoundSocketNames.Remove(SocketName) == 0)
	{
		SetLastSocketError(ESocketErrors::SE_ENOTSOCK);
		return false;
	}

	return true;
}
