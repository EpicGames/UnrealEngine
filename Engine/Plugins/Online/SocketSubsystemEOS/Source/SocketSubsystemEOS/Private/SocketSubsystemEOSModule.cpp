// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystemEOSModule.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FSocketSubsystemEOSModule, SocketSubsystemEOS);

void FSocketSubsystemEOSModule::StartupModule()
{

}

void FSocketSubsystemEOSModule::ShutdownModule()
{

}
