// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class FSocketSubsystemEOSModule: public IModuleInterface
{
public:
	FSocketSubsystemEOSModule() = default;
	~FSocketSubsystemEOSModule() = default;

private:
	// ~Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~End IModuleInterface
};