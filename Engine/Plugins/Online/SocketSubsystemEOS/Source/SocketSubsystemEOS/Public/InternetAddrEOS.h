// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EOSShared.h"
#include "IPAddress.h"

#if WITH_EOS_P2P
#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_common.h"
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogSocketSubsystemEOS, Log, All);

#define EOS_SOCKET_NAME_SIZE 33

class FInternetAddrEOS
	: public FInternetAddr
{
public:
	SOCKETSUBSYSTEMEOS_API FInternetAddrEOS();

	UE_DEPRECATED(5.6, "InSocketName and InChannel have been removed from FInternetAddrEOS, please use the new constructor.")
	SOCKETSUBSYSTEMEOS_API FInternetAddrEOS(const FString& InRemoteUserId, const FString& InSocketName, const int32 InChannel);
	SOCKETSUBSYSTEMEOS_API FInternetAddrEOS(const FString& InRemoteUserId);
#if WITH_EOS_P2P
	UE_DEPRECATED(5.6, "InSocketName and InChannel have been removed from FInternetAddrEOS, please use the new constructor.")
	SOCKETSUBSYSTEMEOS_API FInternetAddrEOS(const EOS_ProductUserId InRemoteUserId, const FString& InSocketName, const int32 InChannel);
	SOCKETSUBSYSTEMEOS_API FInternetAddrEOS(const EOS_ProductUserId InRemoteUserId);
#else
	SOCKETSUBSYSTEMEOS_API FInternetAddrEOS(void* const InRemoteUserId);
#endif
	virtual ~FInternetAddrEOS() = default;

//~ Begin FInternetAddr Interface
	SOCKETSUBSYSTEMEOS_API virtual void SetIp(uint32 InAddr) override;
	SOCKETSUBSYSTEMEOS_API virtual void SetIp(const TCHAR* InAddr, bool& bIsValid) override;
	SOCKETSUBSYSTEMEOS_API virtual void GetIp(uint32& OutAddr) const override;
	SOCKETSUBSYSTEMEOS_API virtual bool IsPortValid(int32 InPort) const override;
	SOCKETSUBSYSTEMEOS_API virtual void SetPort(int32 InPort) override;
	SOCKETSUBSYSTEMEOS_API virtual int32 GetPort() const override;
	SOCKETSUBSYSTEMEOS_API virtual void SetRawIp(const TArray<uint8>& RawAddr) override;
	SOCKETSUBSYSTEMEOS_API virtual TArray<uint8> GetRawIp() const override;
	SOCKETSUBSYSTEMEOS_API virtual void SetAnyAddress() override;
	SOCKETSUBSYSTEMEOS_API virtual void SetBroadcastAddress() override;
	SOCKETSUBSYSTEMEOS_API virtual void SetLoopbackAddress() override;
	SOCKETSUBSYSTEMEOS_API virtual FString ToString(bool bAppendPort) const override;
	SOCKETSUBSYSTEMEOS_API virtual uint32 GetTypeHash() const override;
	SOCKETSUBSYSTEMEOS_API virtual bool IsValid() const override;
	SOCKETSUBSYSTEMEOS_API virtual TSharedRef<FInternetAddr> Clone() const override;
	SOCKETSUBSYSTEMEOS_API virtual FName GetProtocolType() const override;
//~ End FInternetAddr Interface

	inline FInternetAddrEOS& operator=(const FInternetAddrEOS& Other)
	{
		ProductUserId = Other.ProductUserId;
		return *this;
	}

	inline virtual bool operator==(const FInternetAddr& Other) const override
	{
		if (Other.GetProtocolType() == GetProtocolType())
		{
			const FInternetAddrEOS& OtherEOS = static_cast<const FInternetAddrEOS&>(Other);
			return *this == OtherEOS;
		}
		return false;
	}

	inline friend bool operator==(const FInternetAddrEOS& A, const FInternetAddrEOS& B)
	{
		return A.ProductUserId == B.ProductUserId;
	}

	inline friend bool operator!=(const FInternetAddrEOS& A, const FInternetAddrEOS& B)
	{
		return !(A == B);
	}

	inline friend uint32 GetTypeHash(const FInternetAddrEOS& Address)
	{
		return Address.GetTypeHash();
	}

	friend bool operator<(const FInternetAddrEOS& Left, const FInternetAddrEOS& Right)
	{
		return Left.ProductUserId < Right.ProductUserId;
	}

#if WITH_EOS_P2P
	void SetProductUserId(EOS_ProductUserId InProductUserId)
	{
		ProductUserId = InProductUserId;
	}

	EOS_ProductUserId GetProductUserId() const
	{
		return ProductUserId;
	}
#else
	void SetProductUserId(void* InProductUserId)
	{
		ProductUserId = InProductUserId;
	}

	void* GetProductUserId() const
	{
		return ProductUserId;
	}
#endif

private:
#if WITH_EOS_P2P
	EOS_ProductUserId ProductUserId;
#else
	void* ProductUserId;
#endif

	friend class SocketSubsystemEOS;
};
