// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IpNetDriver.h"
#include "NetDriverEOS.generated.h"

#define UE_API SOCKETSUBSYSTEMEOS_API

class ISocketSubsystem;

UCLASS(MinimalAPI, Transient, Config=Engine)
class UNetDriverEOS
	: public UIpNetDriver
{
	GENERATED_BODY()

public:
	UE_API UNetDriverEOS(const FObjectInitializer& ObjectInitializer);

//~ Begin UNetDriver Interface
	UE_API virtual bool IsAvailable() const override;
	UE_API virtual bool InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error) override;
	UE_API virtual bool InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error) override;
	UE_API virtual bool InitListen(FNetworkNotify* InNotify, FURL& LocalURL, bool bReuseAddressAndPort, FString& Error) override;
	UE_API virtual ISocketSubsystem* GetSocketSubsystem() override;
	UE_API virtual void Shutdown() override;
	UE_API virtual int GetClientPort() override;
//~ End UNetDriver Interface

	UE_API UWorld* FindWorld() const;

public:
	UPROPERTY()
	bool bIsPassthrough = false;

	UE_DEPRECATED(5.6, "bIsUsingP2PSockets is deprecated. All code that used it now operates as if it were true")
	UPROPERTY(Config, meta=(DeprecatedProperty, DeprecationMessage="This property is obsolete. All code that used it now operates as if it were true"))
	bool bIsUsingP2PSockets = true;
};

#undef UE_API
