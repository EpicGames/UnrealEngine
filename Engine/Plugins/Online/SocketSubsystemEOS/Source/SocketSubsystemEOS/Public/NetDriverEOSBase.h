// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NetDriverEOS.h"
#include "NetDriverEOSBase.generated.h"

class ISocketSubsystem;

UCLASS(Transient, Config=Engine)
class UE_DEPRECATED(5.6, "UNetDriverEOSBase is deprecated, please use UNetDriverEOS") SOCKETSUBSYSTEMEOS_API UNetDriverEOSBase
	: public UNetDriverEOS
{
	GENERATED_BODY()

public:
};
