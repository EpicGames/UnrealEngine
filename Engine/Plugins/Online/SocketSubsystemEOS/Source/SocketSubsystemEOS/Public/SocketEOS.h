// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Sockets.h"
#include "InternetAddrEOS.h"
#include "EOSSharedTypes.h"

class FOnlineSubsystemEOS;
class FSocketSubsystemEOS;

#if WITH_EOS_P2P
	#include "eos_p2p_types.h"
	#include "eos_types.h"
#endif

class FSocketEOS
	: public FSocket
{
public:
	SOCKETSUBSYSTEMEOS_API FSocketEOS(FSocketSubsystemEOS& SocketSubsystem, const FString& InSocketDescription);
	SOCKETSUBSYSTEMEOS_API virtual ~FSocketEOS();

	//~ Begin FSocket Interface
	virtual bool Shutdown(ESocketShutdownMode Mode) override;
	virtual bool Close() override;
	virtual bool Bind(const FInternetAddr& Addr) override;
	virtual bool Connect(const FInternetAddr& Addr) override;
	virtual bool Listen(int32 MaxBacklog) override;
	virtual bool WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime) override;
	virtual bool HasPendingData(uint32& PendingDataSize) override;
	virtual class FSocket* Accept(const FString& InSocketDescription) override;
	virtual class FSocket* Accept(FInternetAddr& OutAddr, const FString& InSocketDescription) override;
	virtual bool SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination) override;
	virtual bool Send(const uint8* Data, int32 Count, int32& BytesSent) override;
	virtual bool RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None) override;
	virtual bool Recv(uint8* Data, int32 BufferSize, int32& BytesRead, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None) override;
	virtual bool Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime) override;
	virtual ESocketConnectionState GetConnectionState() override;
	virtual void GetAddress(FInternetAddr& OutAddr) override;
	virtual bool GetPeerAddress(FInternetAddr& OutAddr) override;
	virtual bool SetNonBlocking(bool bIsNonBlocking = true) override;
	virtual bool SetBroadcast(bool bAllowBroadcast = true) override;
	virtual bool SetNoDelay(bool bIsNoDelay = true) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress) override;
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress) override;
	virtual bool SetMulticastLoopback(bool bLoopback) override;
	virtual bool SetMulticastTtl(uint8 TimeToLive) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;
	virtual bool SetMulticastInterface(const FInternetAddr& InterfaceAddress) override;
	virtual bool SetReuseAddr(bool bAllowReuse = true) override;
	virtual bool SetLinger(bool bShouldLinger = true, int32 Timeout = 0) override;
	virtual bool SetRecvErr(bool bUseErrorQueue = true) override;
	virtual bool SetSendBufferSize(int32 Size, int32& NewSize) override;
	virtual bool SetReceiveBufferSize(int32 Size, int32& NewSize) override;
	virtual int32 GetPortNo() override;
	//~ End FSocket Interface

	void SetLocalAddress(const FInternetAddrEOS& InLocalAddress);

	void SetSocketName(const FString& InSocketName);

	bool Close(const FInternetAddrEOS& RemoteAddress);

	bool WasClosed(const FInternetAddrEOS& RemoteAddress)
	{
		int32 Index = -1;
		return ClosedRemotes.Find(RemoteAddress, Index);
	}

	void RegisterClosedNotification();

private:

	/** Rebind connection request handler when network goes from offline to online, and cleanup objects when network goes from online to offline */
	void OnNetworkStatusChanged(EOS_ENetworkStatus OldNetworkStatus, EOS_ENetworkStatus NewNetworkStatus);

	/** Used to track our aliveness and make it possible to use the callback interface */
	TSharedPtr<FCallbackBase> CallbackAliveTracker;

	/** Reference to our subsystem */
	FSocketSubsystemEOS& SocketSubsystem;

	/** Our local address; session/port will be invalid when not bound */
	FInternetAddrEOS LocalAddress;
	
	/** Name of the NetDriver that will use the socket */
	FString SocketName;

	/** Are we currently listening? */
	bool bIsListening;

	TArray<FInternetAddrEOS> ClosedRemotes;

#if WITH_EOS_P2P
	typedef TEOSGlobalCallback<EOS_P2P_OnIncomingConnectionRequestCallback, EOS_P2P_OnIncomingConnectionRequestInfo, FCallbackBase> FConnectNotifyCallback;
	typedef TEOSGlobalCallback<EOS_P2P_OnRemoteConnectionClosedCallback, EOS_P2P_OnRemoteConnectionClosedInfo, FCallbackBase> FClosedNotifyCallback;

	FConnectNotifyCallback* ConnectNotifyCallback;
	EOS_NotificationId ConnectNotifyId;

	FClosedNotifyCallback* ClosedNotifyCallback;
	EOS_NotificationId ClosedNotifyId;

	/** Default packet reliability type used in EOS_P2P_SendPacketOptions */
	EOS_EPacketReliability PacketReliability = EOS_EPacketReliability::EOS_PR_UnreliableUnordered;

	/** Delegate fired when the EOS SDK network status changes */
	FDelegateHandle NetworkStatusChangedDelegateHandle;
#endif
};
