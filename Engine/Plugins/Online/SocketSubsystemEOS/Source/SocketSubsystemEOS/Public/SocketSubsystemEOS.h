// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "SocketSubsystem.h"
#include "SocketSubsystemEOSUtils.h"

#if WITH_EOS_P2P
#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif

#include "eos_p2p_types.h"
#endif

#ifndef EOS_SOCKETSUBSYSTEM
#define EOS_SOCKETSUBSYSTEM FName(TEXT("EOS"))
#endif

class UNetConnectionEOS;
class FSocket;
class FResolveInfoCached;
class FResolveInfo;
class FInternetAddr;
class FInternetAddrEOS;
class FSocketEOS;

typedef TSet<uint8> FChannelSet;

class FSocketSubsystemEOS
	: public ISocketSubsystem
{
public:
	SOCKETSUBSYSTEMEOS_API FSocketSubsystemEOS(IEOSPlatformHandlePtr InPlatformHandle, ISocketSubsystemEOSUtilsPtr InUtils);
	SOCKETSUBSYSTEMEOS_API virtual ~FSocketSubsystemEOS();

//~ Begin ISocketSubsystem Interface
	virtual bool Init(FString& Error) override;
	virtual void Shutdown() override;
	virtual FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType) override;
	virtual FResolveInfoCached* CreateResolveInfoCached(TSharedPtr<FInternetAddr> Addr) const override;
	virtual void DestroySocket(FSocket* Socket) override;
	virtual FAddressInfoResult GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName = nullptr, EAddressInfoFlags QueryFlags = EAddressInfoFlags::Default, const FName ProtocolTypeName = NAME_None, ESocketType SocketType = ESocketType::SOCKTYPE_Unknown) override;
	virtual bool RequiresChatDataBeSeparate() override;
	virtual bool RequiresEncryptedPackets() override;
	virtual bool GetHostName(FString& HostName) override;
	virtual TSharedRef<FInternetAddr> CreateInternetAddr() override;
	virtual TSharedPtr<FInternetAddr> GetAddressFromString(const FString& InString) override;
	virtual bool HasNetworkDevice() override;
	virtual const TCHAR* GetSocketAPIName() const override;
	virtual ESocketErrors GetLastErrorCode() override;
	virtual ESocketErrors TranslateErrorCode(int32 Code) override;
	virtual bool GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAdresses) override;
	virtual TSharedRef<FInternetAddr> GetLocalBindAddr(FOutputDevice& Out) override;
	virtual TArray<TSharedRef<FInternetAddr>> GetLocalBindAddresses() override;
	TSharedRef<FInternetAddr> GetLocalBindAddr(const UWorld* const OwningWorld, FOutputDevice& Out);
	virtual bool IsSocketWaitSupported() const override;
//~ End ISocketSubsystem Interface

	void SetLastSocketError(const ESocketErrors NewSocketError);

#if WITH_EOS_P2P
	EOS_HP2P GetP2PHandle();
	EOS_ProductUserId GetLocalUserId();
#endif

	/**
	 * Bind our socket name and ensure no other connections are using this combination
	 *
	 * @return True if we bound successfully, false if not and LastSocketError will be set with a reason
	 */
	bool BindSocketName(const FString& SocketName);

	/**
	 * Unbind a previously bound socket name.
	 *
	 * @return True if we released the port successfully, false if not and LastSocketError will be set with a reason
	 */
	bool UnbindSocketName(const FString& SocketName);

	/**
	 * Returns the FSocketSubsytemEOS instance corresponding with the UWorld* parameter.
	 * If no entry exists for the given UWorld*, it will add it if a valid instance is found.
	 * 
	 * @param InWorld UWorld* to be used in the search
	 * @return The FSocketSubsystemEOS instance associated with InWorld
	 */
	FSocketSubsystemEOS* GetSocketSubsystemForWorld(UWorld* InWorld);

	/**
	 * Returns the utils for this SocketSubsystemEOS
	 *
	 * @return The ISocketSubsystemEOSUtilsPtr for this SocketSubsystemEOS
	 */
	ISocketSubsystemEOSUtilsPtr GetSocketSubsystemEOSUtils() { return Utils; }

private:
	/** Removes the FSocketSubsystemEOS instance from SocketSubsystemEOSInstances and all related entries from SocketSubsystemEOSPerWorldMap */
	void RemoveFromStaticContainers();

private:
#if WITH_EOS_P2P
	EOS_HP2P P2PHandle = nullptr;
#endif
	ISocketSubsystemEOSUtilsPtr Utils;

	/** All sockets allocated by this subsystem */
	TArray<TUniquePtr<FSocketEOS>> TrackedSockets;

	/** All bound socket names */
	TSet<FString> BoundSocketNames;

	/** The last error we received */
	ESocketErrors LastSocketError;

	/** Static array of all instances of FSocketSubsystemEOS running concurrently (can be more than one in PIE) */
	static TArray<FSocketSubsystemEOS*> SocketSubsystemEOSInstances;

	/** Static map of world to corresponding FSocketSubsystemeOS instance */
	static TMap<UWorld*, FSocketSubsystemEOS*> SocketSubsystemEOSPerWorldMap;
};
