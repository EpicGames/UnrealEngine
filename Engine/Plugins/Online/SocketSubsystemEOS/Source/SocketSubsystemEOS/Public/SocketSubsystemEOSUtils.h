// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EOSShared.h"

#if WITH_EOS_P2P
#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif

#include "eos_common.h"
#endif

class ISocketSubsystemEOSUtils
{
public:
	virtual ~ISocketSubsystemEOSUtils() = default;

#if WITH_EOS_P2P
	virtual EOS_ProductUserId GetLocalUserId() = 0;
#endif
	virtual FString GetSessionId() = 0;

	virtual FName GetSubsystemInstanceName() = 0;

	virtual bool IsLoggedIn() = 0;
};
typedef TSharedPtr<ISocketSubsystemEOSUtils, ESPMode::ThreadSafe> ISocketSubsystemEOSUtilsPtr;
