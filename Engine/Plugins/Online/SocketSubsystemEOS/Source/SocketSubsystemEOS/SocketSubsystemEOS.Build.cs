// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using UnrealBuildTool;

public class SocketSubsystemEOS : ModuleRules
{
	public SocketSubsystemEOS(ReadOnlyTargetRules Target) : base(Target)
	{
		Type = ModuleType.CPlusPlus;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"EOSShared",
				"NetCore",
				"Sockets",
				"OnlineSubsystemUtils"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreOnline",
				"CoreUObject",
				"EOSShared",
				"EOSSDK"
			}
		);
	}
}
